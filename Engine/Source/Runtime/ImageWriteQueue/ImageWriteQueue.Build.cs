// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class ImageWriteQueue : ModuleRules
{
	protected virtual bool bUseGIOThreadPool => Target.IsInPlatformGroup(UnrealPlatformGroup.IOS) || Target.IsInPlatformGroup(UnrealPlatformGroup.Android);

	public ImageWriteQueue(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"ImageCore",
				"ImageWrapper",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Engine",
				"RenderCore",
				"RHI",
			}
		);

		PrivateDefinitions.Add("UE_IWQ_USE_GIOTHREADPOOL=" + (bUseGIOThreadPool ? "1" : "0"));
	}
}
