// Copyright Epic Games, Inc. All Rights Reserved.

#include "ImageWriteBlueprintLibrary.h"
#include "Engine/Texture.h"
#include "ImageWriteQueue.h"
#include "Modules/ModuleManager.h"
#include "Async/Async.h"
#include "ImagePixelData.h"
#include "Engine/Texture2D.h"
#include "Engine/TextureRenderTarget2D.h"
#include "TextureResource.h"
#include "RenderingThread.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ImageWriteBlueprintLibrary)

EImageFormat ImageFormatFromDesired(EDesiredImageFormat In)
{
	switch (In)
	{
		case EDesiredImageFormat::PNG: return EImageFormat::PNG;
		case EDesiredImageFormat::JPG: return EImageFormat::JPEG;
		case EDesiredImageFormat::BMP: return EImageFormat::BMP;
		case EDesiredImageFormat::EXR: return EImageFormat::EXR;
	}

	return EImageFormat::BMP;
}


bool UImageWriteBlueprintLibrary::ResolvePixelData(UTexture* InTexture, const FOnPixelsReady& OnPixelsReady)
{
	if (!InTexture)
	{
		FFrame::KismetExecutionMessage(TEXT("Invalid texture supplied."), ELogVerbosity::Error);
		return false;
	}

	EPixelFormat Format = PF_Unknown;

	if (UTextureRenderTarget2D* RT2D = Cast<UTextureRenderTarget2D>(InTexture))
	{
		Format = RT2D->GetFormat();
	}
	else if (UTexture2D* Texture2D = Cast<UTexture2D>(InTexture))
	{
		Format = Texture2D->GetPixelFormat();
	}

	switch (Format)
	{
	default:
		FFrame::KismetExecutionMessage(TEXT("Unsupported texture format."), ELogVerbosity::Error);
		return false;

	case PF_FloatRGBA:
	case PF_A32B32G32R32F:
	case PF_R8G8B8A8:
	case PF_B8G8R8A8:
		break;
	}

	FTextureResource* TextureResource = InTexture->GetResource();
	ENQUEUE_RENDER_COMMAND(ResolvePixelData)(
		[TextureResource, OnPixelsReady](FRHICommandListImmediate& RHICmdList)
		{
			FTextureRHIRef Texture2D = TextureResource->TextureRHI ? TextureResource->TextureRHI->GetTexture2D() : nullptr;
			if (!Texture2D)
			{
				OnPixelsReady(nullptr);
				return;
			}

			FIntRect SourceRect(0, 0, Texture2D->GetDesc().Extent.X, Texture2D->GetDesc().Extent.Y);
			switch (Texture2D->GetFormat())
			{
				case PF_FloatRGBA:
				{
					TArray<FFloat16Color> RawPixels;
					RawPixels.SetNum(SourceRect.Width() * SourceRect.Height());
					RHICmdList.ReadSurfaceFloatData(Texture2D, SourceRect, RawPixels, (ECubeFace)0, 0, 0);

					TUniquePtr<TImagePixelData<FFloat16Color>> PixelData = MakeUnique<TImagePixelData<FFloat16Color>>(SourceRect.Size(), TArray64<FFloat16Color>(MoveTemp(RawPixels)));

					if (PixelData->IsDataWellFormed())
					{
						OnPixelsReady(MoveTemp(PixelData));
						return;
					}

					break;
				}

				case PF_A32B32G32R32F:
				{
					FReadSurfaceDataFlags ReadDataFlags(RCM_MinMax);
					ReadDataFlags.SetLinearToGamma(false);


					TArray<FLinearColor> RawPixels;
					RawPixels.SetNum(SourceRect.Width() * SourceRect.Height());
					RHICmdList.ReadSurfaceData(Texture2D, SourceRect, RawPixels, ReadDataFlags);
					TUniquePtr<TImagePixelData<FLinearColor>> PixelData = MakeUnique<TImagePixelData<FLinearColor>>(SourceRect.Size(), TArray64<FLinearColor>(MoveTemp(RawPixels)));
					if (PixelData->IsDataWellFormed())
					{
						OnPixelsReady(MoveTemp(PixelData));
						return;
					}

					break;
				}

				case PF_R8G8B8A8:
				case PF_B8G8R8A8:
				{
					FReadSurfaceDataFlags ReadDataFlags;
					ReadDataFlags.SetLinearToGamma(false);

					TArray<FColor> RawPixels;
					RawPixels.SetNum(SourceRect.Width() * SourceRect.Height());
					RHICmdList.ReadSurfaceData(Texture2D, SourceRect, RawPixels, ReadDataFlags);

					TUniquePtr<TImagePixelData<FColor>> PixelData = MakeUnique<TImagePixelData<FColor>>(SourceRect.Size(), TArray64<FColor>(MoveTemp(RawPixels)));
					if (PixelData->IsDataWellFormed())
					{
						OnPixelsReady(MoveTemp(PixelData));
						return;
					}

					break;
				}

				default:
					break;
			}

			OnPixelsReady(nullptr);
		}
	);

	return true;
}


void UImageWriteBlueprintLibrary::ExportToDisk(UTexture* InTexture, const FString& InFilename, const FImageWriteOptions& InOptions)
{
	//~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~
	// Care should be take here to ensure that completion callbacks are always called from each 'exit' point
	// If they user has passed in a callback they *expect* it to be called regardless of the error that was emitted
	//~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~

	// Capture the native completion callback and the dynamic completion callback and call them both from the main
	// thread when the image write task has finished, or an error occurred
	TFunction<void(bool)> OnCompleteWrapper = [NativeCB = InOptions.NativeOnComplete, DynamicCB = InOptions.OnComplete](bool bSuccess)
	{
		if (NativeCB)
		{
			NativeCB(bSuccess);
		}
		DynamicCB.ExecuteIfBound(bSuccess);
	};

	// In the case of an error, we always call the error callbacks in a latent manner to ensure that we always trigger the calback outside of this 
	// function - this ensures the calling context deterministic for whatever is handling the completion.
	if (!InTexture || !InTexture->GetResource() || !InTexture->GetResource()->TextureRHI)
	{
		FFrame::KismetExecutionMessage(TEXT("Invalid texture supplied."), ELogVerbosity::Error);
		AsyncTask(ENamedThreads::GameThread, [OnCompleteWrapper] { OnCompleteWrapper(false); });
		return;
	}

	FTextureRHIRef Texture2D = InTexture->GetResource()->TextureRHI->GetTexture2D();
	if (!Texture2D)
	{
		FFrame::KismetExecutionMessage(TEXT("Invalid texture supplied."), ELogVerbosity::Error);
		AsyncTask(ENamedThreads::GameThread, [OnCompleteWrapper] { OnCompleteWrapper(false); });
		return;
	}

	switch (Texture2D->GetFormat())
	{
	default:
		FFrame::KismetExecutionMessage(TEXT("Unsupported texture format."), ELogVerbosity::Error);
		AsyncTask(ENamedThreads::GameThread, [OnCompleteWrapper] { OnCompleteWrapper(false); });
		return;

	case PF_FloatRGBA:
	case PF_A32B32G32R32F:
		if (InOptions.Format != EDesiredImageFormat::EXR)
		{
			FFrame::KismetExecutionMessage(TEXT("Only EXR export is currently supported for PF_FloatRGBA and PF_A32B32G32R32F formats."), ELogVerbosity::Error);
			AsyncTask(ENamedThreads::GameThread, [OnCompleteWrapper] { OnCompleteWrapper(false); });
			return;
		}
		break;

	case PF_R8G8B8A8:
	case PF_B8G8R8A8:
		break;
	}

	struct FCommandParameters
	{
		FCommandParameters()
		{
			ImageWriteQueue = &FModuleManager::Get().LoadModuleChecked<IImageWriteQueueModule>("ImageWriteQueue").GetWriteQueue();
		}

		/** The filename to export to */
		FString Filename;
		/** The image format to write as */
		EDesiredImageFormat Format;
		/** A compression quality to use for the image (EImageCompressionQuality for EXRs, or a value between 0 and 100) */
		int32 CompressionQuality;
		/** true to overwrite the file if it already exists, false otherwise */
		bool bOverwriteFile;
		/** true for async, false to block until the file has been written out (will block both the render thread and the main thread until the render target has been fully exported) */
		bool bAsync;

		/** Called when the image write task has completed */
		TFunction<void(bool)> OnComplete;

		/** The image write queue to use for exporting the image */
		IImageWriteQueue* ImageWriteQueue;
		/** A shared promise that will be set when the image task has been dispatched */
		TSharedPtr<TPromise<void>, ESPMode::ThreadSafe> SharedPromise;
	};

	FCommandParameters Params;
	Params.Filename = InFilename;
	Params.OnComplete = MoveTemp(OnCompleteWrapper);
	Params.Format = InOptions.Format;
	Params.CompressionQuality = InOptions.CompressionQuality;
	Params.bOverwriteFile = InOptions.bOverwriteFile;
	Params.bAsync = InOptions.bAsync;
	if (!Params.bAsync)
	{
		Params.SharedPromise = MakeShared<TPromise<void>, ESPMode::ThreadSafe>();
	}

	auto ProcessPixels = [Params](TUniquePtr<FImagePixelData>&& PixelData)
	{
		TFuture<bool> DispatchedTask;

		if (PixelData.IsValid())
		{
			TUniquePtr<FImageWriteTask> ImageTask = MakeUnique<FImageWriteTask>();

			ImageTask->PixelData = MoveTemp(PixelData);
			ImageTask->Format = ImageFormatFromDesired(Params.Format);
			ImageTask->OnCompleted = Params.OnComplete;
			ImageTask->Filename = Params.Filename;
			ImageTask->bOverwriteFile = Params.bOverwriteFile;
			ImageTask->CompressionQuality = Params.CompressionQuality;

			DispatchedTask = Params.ImageWriteQueue->Enqueue(MoveTemp(ImageTask));
		}

		// Wait for the task to finish if we're sync
		if (!Params.bAsync)
		{
			// If not async, wait for the dispatched task to complete, then set the outer promise so the calling thread can return the result
			if (DispatchedTask.IsValid())
			{
				DispatchedTask.Wait();
			}

			Params.SharedPromise->SetValue();
		}
	};

	if (ResolvePixelData(InTexture, ProcessPixels) && !InOptions.bAsync)
	{
		Params.SharedPromise->GetFuture().Wait();
	}
}
