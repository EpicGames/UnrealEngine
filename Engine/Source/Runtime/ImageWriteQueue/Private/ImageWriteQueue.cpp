// Copyright Epic Games, Inc. All Rights Reserved.

#include "ImageWriteQueue.h"
#include "HAL/IConsoleManager.h"
#include "HAL/ThreadSafeBool.h"
#include "HAL/FileManager.h"
#include "Misc/Paths.h"
#include "Misc/ScopeLock.h"
#include "Misc/FileHelper.h"
#include "Async/Async.h"
#include "Modules/ModuleManager.h"
#include "IImageWrapperModule.h"
#include "Misc/QueuedThreadPoolWrapper.h"

DEFINE_LOG_CATEGORY(LogImageWriteQueue);

static TAutoConsoleVariable<int32> CVarImageWriteQueueMaxConcurrency(
	TEXT("ImageWriteQueue.MaxConcurrency"),
	-1,
	TEXT("The maximum number of async image writes allowable at any given time.")
	TEXT("Default is to use the number of cores available."),
	ECVF_Default);

static TAutoConsoleVariable<int32> CVarImageWriteQueueMaxQueueSize(
	TEXT("ImageWriteQueue.MaxQueueSize"),
	-1,
	TEXT("The maximum number of queued image write tasks allowable before the queue will block when adding more.")
	TEXT("Default is to use 4 times the number of cores available or 16 when multithreading is disabled on the command line."),
	ECVF_Default);

/**
 * Struct defining particular 'fence' within the queue
 */
struct FImageWriteFence
{
	FImageWriteFence(uint32 InID, uint32 InCount, TPromise<void>&& InCompleted, TFunction<void()> InOnCompleted)
		: ID(InID)
		, Count(InCount)
		, Completed(MoveTemp(InCompleted))
		, OnCompleted(MoveTemp(InOnCompleted))
	{
	}

	FImageWriteFence(FImageWriteFence&&) = default;
	FImageWriteFence(const FImageWriteFence&) = delete;

	FImageWriteFence& operator=(FImageWriteFence&&) = default;
	FImageWriteFence& operator=(const FImageWriteFence&) = delete;

	/** A unique identifier for this fence, any tasks enqueued before this fence will have an ID <= this fence's ID */
	uint32 ID;

	/** The number of tasks currently dispatched with an ID <= this fence */
	uint32 Count;

	/** A promise to fulfil when this fence has been reached */
	TPromise<void> Completed;

	/** A callback to call on the game thread when this fence has been reached */
	TFunction<void()> OnCompleted;
};

/** Private implementation of the write queue */
class FImageWriteQueue : public IImageWriteQueue
{
public:

	FImageWriteQueue();
	~FImageWriteQueue();

public:

	/* ~ Begin IImageWriteQueue interface */
	virtual TFuture<bool> Enqueue(TUniquePtr<IImageWriteTaskBase>&& InTask, bool bBlockIfAtCapacity = true) override;
	virtual TFuture<void> CreateFence(const TFunction<void()>& InOnFenceReached = TFunction<void()>()) override;
	virtual int32 GetNumPendingTasks() const override;
	/* ~ End IImageWriteQueue interface */

public:

	/**
	 * (thread-safe) Called from a task when it has been completed.
	 * 
	 * @param FenceID      The fence ID that the task was created under
	 */
	void OnTaskCompleted(uint32 FenceID);

	/**
	 * (thread-safe) Called from the module when this queue should start shutting down.
	 * Prevents any susequent tasks from being enqueued
	 */
	void BeginShutdown();

private:

	/**
	 * Called when any cvar in the engine is changed. Causes a recreation of the thread pool if necessary.
	 */
	void OnCVarsChanged();

	/**
	 * Ensure that the thread pool is set up with the correct number of pooled threads
	 */
	void RecreateThreadPool();

	/**
	 * (thread-safe) Decrement the number of tasks pending for any fence ID that is >= the fence specified
	 * 
	 * @param FenceID      The fence ID to decrement
	 */
	void DecrementFence(uint32 FenceID);

private:

	/** Atomic count of currently pending (and in progress) tasks */
	TAtomic<int32> NumPendingTasks;
	/** Atomic cache of the maximum number of allowable queued (and in progress) tasks */
	TAtomic<int32> MaxQueueSize;

	/** Auto-reset event that is signalled every time a task completes */
	FEvent* OnTaskCompletedEvent;


	/* ~~~ Begin ThreadPoolMutex protection ~~~*/
	FCriticalSection ThreadPoolMutex;
	/** True when ThreadPool is an allocated thread pool that must be deleted on shutdown */
	bool bOwnedThreadPool;
	/** Thread pool to queue tasks within - pool size set to the max concurrency cvar */
	FQueuedThreadPool* ThreadPool;
	/* ~~~ End ThreadPoolMutex protection ~~~*/


	/* ~~~ Begin FenceMutex protection ~~~*/
	FCriticalSection FenceMutex;
	/** Array of fences that are still waiting to be reached */
	TArray<FImageWriteFence> PendingFences;
	/** Serial ID of the next fence that should be returned. Starts at 0, increments each time a fence is created. */
	uint32 CurrentFenceID;
	/** Incrementing count of the number of tasks that have been enqueued since the last fence was created. */
	uint32 CurrentFenceCount;
	/* ~~~ End FenceMutex protection ~~~*/


	/** Delegate handle for a consolve variable sink */
	FConsoleVariableSinkHandle CVarSinkHandle;

	/** Set when we are pending shutdown and no new tasks should be added */
	FThreadSafeBool bPendingShutdown;
};

/** Implementation of the queued work that just writes a task */
class FQueuedImageWrite : public IQueuedWork
{
public:

	FQueuedImageWrite(uint32 InFenceID, FImageWriteQueue* InOwner, TUniquePtr<IImageWriteTaskBase>&& InTask, TPromise<bool>&& InPromise)
		: FenceID(InFenceID)
		, Owner(InOwner)
		, Task(MoveTemp(InTask))
		, Promise(MoveTemp(InPromise))
	{}

	/** Perform the work on the current thread, and delete this object when done */
	void RunTaskOnCurrentThread()
	{
		// Perform any compression, conversion and pixel processing, then write the image to disk
		bool bSuccess = Task->RunTask();

		Promise.SetValue(bSuccess);

		// Inform the owning queue that a task was completed with this task's fence ID
		Owner->OnTaskCompleted(FenceID);
		delete this;
	}

private:

	/** Called on a pooled thread when this work is to be performed */
	virtual void DoThreadedWork() override
	{
		RunTaskOnCurrentThread();
	}

	virtual void Abandon() override
	{
		Promise.SetValue(false);

		// Inform the owning queue that a task was completed with this task's fence ID
		Owner->OnTaskCompleted(FenceID);
		delete this;
	}

private:
	/** The fence ID context that this task was dispatched within */
	uint32 FenceID;
	/** The owning queue that dispatched this task */
	FImageWriteQueue* Owner;
	/** The task itself */
	TUniquePtr<IImageWriteTaskBase> Task;
	/** A promise to fulfil when this task has been performed or abandoned */
	TPromise<bool> Promise;
};

FImageWriteQueue::FImageWriteQueue()
	: NumPendingTasks(0)
	, bOwnedThreadPool(false)
	, ThreadPool(nullptr)
	, CurrentFenceID(0)
	, CurrentFenceCount(0)
	, bPendingShutdown(false)
{
	// Ensure that the image wrapper module is loaded - required for GImageWrappers
	FModuleManager::Get().LoadModuleChecked<IImageWrapperModule>("ImageWrapper");

	// Allocate the task completion event
	bool bManualResetEvent = false;
	OnTaskCompletedEvent = FPlatformProcess::GetSynchEventFromPool(bManualResetEvent);

	// Create the cvar sink and set up the thread pool
	CVarSinkHandle = IConsoleManager::Get().RegisterConsoleVariableSink_Handle(FConsoleCommandDelegate::CreateRaw(this, &FImageWriteQueue::OnCVarsChanged));
	OnCVarsChanged();
}

FImageWriteQueue::~FImageWriteQueue()
{
	check(bPendingShutdown && NumPendingTasks == 0);
	FPlatformProcess::ReturnSynchEventToPool(OnTaskCompletedEvent);

	IConsoleManager::Get().UnregisterConsoleVariableSink_Handle(CVarSinkHandle);

	if (bOwnedThreadPool)
	{
		ThreadPool->Destroy();
		delete ThreadPool;
	}
}

void FImageWriteQueue::OnCVarsChanged()
{
	RecreateThreadPool();
	const int32 ConfiguredMaxQueueSize = CVarImageWriteQueueMaxQueueSize.GetValueOnAnyThread();
	MaxQueueSize = ConfiguredMaxQueueSize == -1 ? (ThreadPool ? ThreadPool->GetNumThreads() * 4 : 16) : ConfiguredMaxQueueSize;
}

void FImageWriteQueue::RecreateThreadPool()
{
	if (!FPlatformProcess::SupportsMultithreading())
	{
		return;
	}

	// Prevent any other tasks being dispatched
	FScopeLock ScopeLock(&ThreadPoolMutex);

#if UE_IWQ_USE_GIOTHREADPOOL
	// To avoid spawning extra threads use global IO thread on mobile 
	const int32 MaxConcurrency = GIOThreadPool->GetNumThreads();
#else
	const int32 ConfiguredMaxConcurrency = CVarImageWriteQueueMaxConcurrency.GetValueOnAnyThread();
	const int32 MaxConcurrency = ConfiguredMaxConcurrency == -1 ? FPlatformMisc::NumberOfCores() : ConfiguredMaxConcurrency;
#endif

	if (ThreadPool && MaxConcurrency != ThreadPool->GetNumThreads())
	{
		CreateFence().Wait();

		if (bOwnedThreadPool)
		{
			ThreadPool->Destroy();
			delete ThreadPool;
			ThreadPool = nullptr;
		}
		else
		{
			check(ThreadPool == GIOThreadPool);
			ThreadPool = nullptr;
		}
	}

	if (!ThreadPool)
	{
		if (MaxConcurrency == GIOThreadPool->GetNumThreads())
		{
			// Use the global IO thread pool if possible
			bOwnedThreadPool = false;
			ThreadPool = GIOThreadPool;
		}
		else if (GThreadPool)
		{
			// Use a simple wrapper to limit concurrency and reuse threads we already have
			bOwnedThreadPool = true;
			ThreadPool = new FQueuedThreadPoolWrapper(GThreadPool, MaxConcurrency);
		}
		else
		{
			// Create a new thread pool as a last resort
			bOwnedThreadPool = true;
			ThreadPool = FQueuedThreadPool::Allocate();
			verify(ThreadPool->Create(MaxConcurrency, 5 * 1024));
		}
	}
}

void FImageWriteQueue::DecrementFence(uint32 FenceID)
{
	FScopeLock FenceLock(&FenceMutex);

	// If this fence ID is the current fence context, there cannot be any fences dependent upon this task
	if (FenceID == CurrentFenceID)
	{
		--CurrentFenceCount;
		return;
	}

	int32 LastCompletedFenceIndex = -1;

	// Iterate the pending fences in order,
	// decrement the fence count for this ID and
	// gather the last consecutive completed fence index (with a count of 0)
	for (int32 Index = 0; Index < PendingFences.Num(); ++Index)
	{
		FImageWriteFence& Fence = PendingFences[Index];

		// If the current fence depends upon the ID supplied, and has outstanding tasks, we can't have reached any fence beyond it
		if (Fence.ID > FenceID && Fence.Count > 0)
		{
			break;
		}

		// If this is the supplied fence ID, decrement its count
		if (Fence.ID == FenceID)
		{
			--Fence.Count;
		}

		// If the previous fence has been reached, and so has this, increment the last completed fence index
		if (Index == LastCompletedFenceIndex + 1 && Fence.Count == 0)
		{
			++LastCompletedFenceIndex;
		}
	}

	// If there is any chain of consecutive fences that have been reached, complete them all now
	if (LastCompletedFenceIndex >= 0)
	{
		for (int32 Index = 0; Index <= LastCompletedFenceIndex; ++Index)
		{
			FImageWriteFence& Fence = PendingFences[Index];
			check(Fence.Count == 0);

			Fence.Completed.SetValue();
			if (Fence.OnCompleted)
			{
				AsyncTask(ENamedThreads::GameThread, [LocalOnCompleted = MoveTemp(Fence.OnCompleted)] { LocalOnCompleted(); });
			}
		}

		PendingFences.RemoveAt(0, LastCompletedFenceIndex+1, EAllowShrinking::No);
	}
}

void FImageWriteQueue::OnTaskCompleted(uint32 FenceID)
{
	DecrementFence(FenceID);

	--NumPendingTasks;
	OnTaskCompletedEvent->Trigger();
}

void FImageWriteQueue::BeginShutdown()
{
	bPendingShutdown = true;

	CreateFence().Wait();
}

int32 FImageWriteQueue::GetNumPendingTasks() const
{
	return NumPendingTasks;
}

TFuture<void> FImageWriteQueue::CreateFence(const TFunction<void()>& InOnFenceReached)
{
	TPromise<void> Promise;
	TFuture<void> Future = Promise.GetFuture();

	FScopeLock FenceLock(&FenceMutex);
	if (PendingFences.Num() == 0 && CurrentFenceCount == 0)
	{
		// The queue is completely empty, return immediately
		Promise.SetValue();
		if (InOnFenceReached)
		{
			AsyncTask(ENamedThreads::GameThread, [InOnFenceReached] { InOnFenceReached(); });
		}
	}
	else
	{
		// Move the promise into the write fence
		PendingFences.Emplace(CurrentFenceID, CurrentFenceCount, MoveTemp(Promise), CopyTemp(InOnFenceReached));

		// Reset the current fence context
		++CurrentFenceID;
		CurrentFenceCount = 0;
	}

	return Future;
}

TFuture<bool> FImageWriteQueue::Enqueue(TUniquePtr<IImageWriteTaskBase>&& InTask, bool bBlockIfAtCapacity)
{
	if (!ensureMsgf(!bPendingShutdown, TEXT("Cannot issue a new image write command while the queue is shutting down.")))
	{
		return TFuture<bool>();
	}

	// Block if the queue is at capacity
	if (bBlockIfAtCapacity)
	{
		while (NumPendingTasks >= MaxQueueSize)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FImageWriteQueue::EnqueueWait)
			OnTaskCompletedEvent->Wait();
		}
	}
	else if (NumPendingTasks >= MaxQueueSize)
	{
		UE_LOGF(LogImageWriteQueue, Warning, "Cannot issue a new image write command because the Queue is at max capacity.");
		return TFuture<bool>();
	}

	TPromise<bool> Promise;
	TFuture<bool> Future = Promise.GetFuture();

	// Get the fence metrics for this task
	uint32 ThisTaskFenceID;
	{
		FScopeLock FenceLock(&FenceMutex);
		ThisTaskFenceID = CurrentFenceID;
		++CurrentFenceCount;
	}

	FQueuedImageWrite* NewTask = new FQueuedImageWrite(ThisTaskFenceID, this, MoveTemp(InTask), MoveTemp(Promise));

	// The thread pool will be nullptr where the platform does not support multi-threading,
	// If so, dispatch and execute the task immediately
	if (!ThreadPool)
	{
		// RunTaskOnCurrentThread deletes itself
		NewTask->RunTaskOnCurrentThread();

		// NewTask is now invalid
	}
	else
	{
		// Dispatch the queued work - must operate under a lock since the thread pool can change at runtime in response to CVar changes
		FScopeLock ThreadPoolLock(&ThreadPoolMutex);
		ThreadPool->AddQueuedWork(NewTask);
	}

	++NumPendingTasks;

	return Future;
}

class FImageWriteQueueModule : public IImageWriteQueueModule
{
	virtual void StartupModule() override
	{
		Queue = MakeUnique<FImageWriteQueue>();
	}

	virtual void PreUnloadCallback() override
	{
		Queue->BeginShutdown();
	}

	virtual void ShutdownModule() override
	{
		Queue->BeginShutdown();
		Queue.Reset();
	}

	virtual IImageWriteQueue& GetWriteQueue() override
	{
		return *Queue;
	}

	TUniquePtr<FImageWriteQueue> Queue;
};

IMPLEMENT_MODULE(FImageWriteQueueModule, ImageWriteQueue)

