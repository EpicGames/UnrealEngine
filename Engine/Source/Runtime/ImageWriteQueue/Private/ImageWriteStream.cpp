// Copyright Epic Games, Inc. All Rights Reserved.

#include "ImageWriteStream.h"
#include "Misc/ScopeLock.h"

void FImageStreamEndpoint::PipeImage(TUniquePtr<FImagePixelData>&& InOwnedImage)
{
	OnImageReceived(MoveTemp(InOwnedImage));
}

void FImagePixelPipe::Push(TUniquePtr<FImagePixelData>&& InImagePixelData)
{
	if (!InImagePixelData.IsValid())
	{
		return;
	}

	FScopeLock Lock(&EndPointLock);

	// Pass onto end points that need a copy of the image, making as few copies as possible
	for (int32 Index = 0; Index < EndPoints.Num(); ++Index)
	{
		if (Index == EndPoints.Num() - 1)
		{
			// Pass ownership of the the input parameter to the last pipe
			EndPoints[Index]->PipeImage(MoveTemp(InImagePixelData));
			// InImagePixelData is now dead
		}
		else
		{
			EndPoints[Index]->PipeImage(InImagePixelData->CopyImageData());
		}
	}
}

void FImagePixelPipe::AddEndpoint(const TFunction<void(TUniquePtr<FImagePixelData>&& )>& InHandler)
{
	typedef TFunction<void(TUniquePtr<FImagePixelData>&&)> FLocalHandler;

	struct FFunctorEndpoint : FImageStreamEndpoint
	{
		FLocalHandler Handler;
		FFunctorEndpoint(const FLocalHandler& InLocalHandler) : Handler(InLocalHandler) {}

		virtual void OnImageReceived(TUniquePtr<FImagePixelData>&& InImageData) override
		{
			Handler(MoveTemp(InImageData));
		}
	};

	AddEndpoint(MakeUnique<FFunctorEndpoint>(InHandler));
}

void FImagePixelPipe::AddEndpoint(TUniquePtr<FImageStreamEndpoint>&& InEndpoint)
{
	FScopeLock Lock(&EndPointLock);

	check(InEndpoint);
	EndPoints.Add(MoveTemp(InEndpoint));
}
