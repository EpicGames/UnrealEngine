// Copyright Epic Games, Inc. All Rights Reserved.

#include "ImageWriteTask.h"
#include "ImageWriteQueue.h"
#include "HAL/IConsoleManager.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/FileManager.h"
#include "Misc/Paths.h"
#include "Misc/ScopeLock.h"
#include "Misc/FileHelper.h"
#include "Async/Async.h"
#include "Modules/ModuleManager.h"
#include "IImageWrapperModule.h"

bool FImageWriteTask::RunTask()
{
	bool bSuccess = WriteToDisk();

	if (OnCompleted)
	{
		AsyncTask(ENamedThreads::GameThread, [bSuccess, LocalOnCompleted = MoveTemp(OnCompleted)] { LocalOnCompleted(bSuccess); });
	}

	return bSuccess;
}

void FImageWriteTask::OnAbandoned()
{
	if (OnCompleted)
	{
		AsyncTask(ENamedThreads::GameThread, [LocalOnCompleted = MoveTemp(OnCompleted)] { LocalOnCompleted(false); });
	}
}

void FImageWriteTask::PreProcess()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ImageWriteTask.PreProcess);
	
	for (const FPixelPreProcessor& PreProcessor : PixelPreProcessors)
	{
		// PreProcessors are assumed to be valid. Fetch the Data pointer each time
		// in case a pre-processor changes our pixel data.
		FImagePixelData* Data = PixelData.Get();
		PreProcessor(Data);
	}
}

bool FImageWriteTask::WriteToDisk()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ImageWriteTask.WriteToDisk);

	static FName ImageWrapperName("ImageWrapper");
	IImageWrapperModule& ImageWrapperModule = FModuleManager::LoadModuleChecked<IImageWrapperModule>(ImageWrapperName);
		
	// Ensure that the payload filename has the correct extension for the format
	if ( ImageWrapperModule.GetImageFormatFromExtension(*Filename) != Format )
	{
		const TCHAR* FormatExtension = ImageWrapperModule.GetExtension(Format);
		Filename = FPaths::GetBaseFilename(Filename, false) + TEXT('.') + FormatExtension;
	}

	bool bSuccess = EnsureWritableFile();

	if (bSuccess)
	{
		PreProcess();
		
		FImagePixelData* Data = PixelData.Get();
		FImageView Image = Data->GetImageView();
		if ( Image.RawData == nullptr )
		{
			UE_LOGF(LogImageWriteQueue, Error, "Failed to write image to '%ls'. Couldn't get pixels.", *Filename);
			return false;
		}

		TArray64<uint8> CompressedFile;
		if ( ! ImageWrapperModule.CompressImage(CompressedFile,Format,Image,CompressionQuality) )
		{
			UE_LOGF(LogImageWriteQueue, Error, "Failed to write image to '%ls'. CompressImage failed.", *Filename);
			return false;
		}
		
		uint64 TotalNumberOfBytes, NumberOfFreeBytes;
		if (FPlatformMisc::GetDiskTotalAndFreeSpace(FPaths::GetPath(Filename), TotalNumberOfBytes, NumberOfFreeBytes))
		{
			const uint64 HeadRoom = 1024*1024;
			if (NumberOfFreeBytes < (uint64)CompressedFile.Num() + HeadRoom)
			{
				UE_LOGF(LogImageWriteQueue, Error, "Failed to write image to '%ls'. There is not enough free space on the disk.", *Filename);
				return false;
			}
		}
		else
		{
			uint32 ErrorCode = FPlatformMisc::GetLastError();
			TCHAR ErrorBuffer[1024];
			FPlatformMisc::GetSystemErrorMessage(ErrorBuffer, 1024, ErrorCode);
			UE_LOGF(LogImageWriteQueue, Warning, "Failed to check free space for %ls. Error: %u (%ls)", *FPaths::GetPath(Filename), ErrorCode, ErrorBuffer);
		}

		bSuccess = FFileHelper::SaveArrayToFile(CompressedFile, *Filename);
	}

	if (!bSuccess)
	{
		UE_LOGF(LogImageWriteQueue, Error, "Failed to write image to '%ls'. The pixel format may not be compatible with this image type, or there was an error writing to that filename.", *Filename);
	}

	return bSuccess;
}

bool FImageWriteTask::EnsureWritableFile()
{
	FString Directory = FPaths::GetPath(Filename);

	if (!IFileManager::Get().DirectoryExists(*Directory))
	{
		const bool bRecursive = true;
		IFileManager::Get().MakeDirectory(*Directory, bRecursive);
	}

	// If the file doesn't exist, we're ok to continue
	if (IFileManager::Get().FileSize(*Filename) == -1)
	{
		return true;
	}
	// If we're allowed to overwrite the file, and we deleted it ok, we can continue
	else if (bOverwriteFile && FPlatformFileManager::Get().GetPlatformFile().DeleteFile(*Filename))
	{
		return true;
	}
	// We can't write to the file
	else if (bOverwriteFile)
	{
		UE_LOGF(LogImageWriteQueue, Error, "Failed to write image to '%ls'. Should have overwritten the file, but we failed to delete it.", *Filename);
		return false;
	}
	else
	{
		UE_LOGF(LogImageWriteQueue, Error, "Failed to write image to '%ls'. Shouldn't overwrite the file and it already exists so we can't replace it.", *Filename);
		return false;
	}
}

void FImageWriteTask::AddPreProcessorToSetAlphaOpaque()
{
	struct PreProcessorToSetAlphaOpaque
	{
		void operator()(FImagePixelData* InPixelData)
		{
			InPixelData->SetAlphaOpaque();
		}
	};

	PixelPreProcessors.Emplace( PreProcessorToSetAlphaOpaque() );
}
