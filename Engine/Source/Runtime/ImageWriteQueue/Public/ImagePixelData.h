// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/CoreDefines.h"
#include "Math/IntPoint.h"
#include "IImageWrapper.h"
#include "Templates/UniquePtr.h"
#include "ImageCore.h"

class FFloat16Color;
template<typename PixelType> struct TImagePixelDataTraits;


// todo : use ImageCore ERawImageFormat instead
enum class EImagePixelType
{
	Color,
	Float16,
	Float32,
};

struct IImagePixelDataPayload
{
	virtual ~IImagePixelDataPayload() = default;
};

typedef TSharedPtr<IImagePixelDataPayload, ESPMode::ThreadSafe> FImagePixelPayloadPtr;


// @todo Oodle : use ImageCore FImage instead
//  get rid of this whole class and TImagePixelData too
//	just use an FImage instead
struct FImagePixelData
{
	virtual ~FImagePixelData() = default;
	
	// NOTE : before, U8 would be written to EXR *linear*
	//	now it will get gamma corrected if bSRGB (which is on by default)

	/**
	 * Retrieve the type of this data
	 */
	EImagePixelType GetType() const
	{
		return Type;
	}

	/**
	 * Retrieve the size of this data
	 */
	FIntPoint GetSize() const
	{
		return Size;
	}

	/**
	 * Retrieve the pixel layout of this data
	 */
	ERGBFormat GetPixelLayout() const
	{
		return PixelLayout;
	}

	/**
	 * Retrieve the number of bits per each channel of color in the data
	 */
	uint8 GetBitDepth() const
	{
		return BitDepth;
	}

	/**
	 * Retrieve the number of channels in the data
	 */
	uint8 GetNumChannels() const
	{
		return NumChannels;
	}
	
	/**
	 * Get the pixel data as an FImage
	 */
	FImageView GetImageView() const
	{
		const void* RawPtr    = nullptr;
		int64       SizeBytes = 0;

		FImageView Ret;

		if ( ! GetRawData(RawPtr, SizeBytes) )
		{
			return FImageView();
		}
		
		check( NumChannels == 4 );

		switch(Type)
		{
		case EImagePixelType::Color:
			check(PixelLayout == ERGBFormat::BGRA );
			check(BitDepth == 8 );
			Ret = FImageView( (const FColor *)RawPtr, Size.X, Size.Y, bSRGB ? EGammaSpace::sRGB : EGammaSpace::Linear );
			break;

		case EImagePixelType::Float16:
			check(PixelLayout == ERGBFormat::RGBAF );
			check(BitDepth == 16 );
			Ret = FImageView( (const FFloat16Color *)RawPtr, Size.X, Size.Y );
			break;

		case EImagePixelType::Float32:
			check(PixelLayout == ERGBFormat::RGBAF );
			check(BitDepth == 32 );
			Ret = FImageView( (const FLinearColor *)RawPtr, Size.X, Size.Y );
			break;

		default:
			check(0);
			return FImageView();
		}

		check( Ret.GetImageSizeBytes() == SizeBytes );
		return Ret;
	}

	/**
	* Change the alpha channel to opaque
	*/
	void SetAlphaOpaque()
	{
		FImageCore::SetAlphaOpaque( GetImageView() );
	}

	/**
	 * Check that this data is the size it should be
	 */
	bool IsDataWellFormed() const
	{
		const void* RawPtr    = nullptr;
		int64       SizeBytes = 0;

		return GetRawData(RawPtr, SizeBytes);
	}

	/**
	 * Get the data and its size only if it is well formed
	 */
	bool GetRawData(const void*& OutRawData, int64& OutSizeBytes) const
	{
		const void* RawPtr    = nullptr;
		int64       SizeBytes = 0;

		RetrieveData(RawPtr, SizeBytes);

		int64 FoundTotalSize = int64(Size.X)*int64(Size.Y)*int64(BitDepth / 8)*int64(NumChannels);
		if (RawPtr && SizeBytes == FoundTotalSize)
		{
			OutRawData = RawPtr;
			OutSizeBytes = SizeBytes;
			return true;
		}
		return false;
	}

	/**
	 * Get the size in bytes, regardless of if it is well formed.
	 */
	int64 GetRawDataSizeInBytes() const
	{
		const void* RawPtr    = nullptr;
		int64       SizeBytes = 0;

		RetrieveData(RawPtr, SizeBytes);
		return SizeBytes;
	}

	/**
	 * Copy this whole image buffer. This can be very costly for large images.
	 */
	TUniquePtr<FImagePixelData> CopyImageData() const
	{
		return Copy();
	}

	/**
	 * Move this whole image buffer to a new allocation.
	 */
	TUniquePtr<FImagePixelData> MoveImageDataToNew()
	{
		return Move();
	}

	/**
	* Return a pointer to the Payload stored in this data.
	*/
	template<typename T>
	T* GetPayload() { return static_cast<T*>(Payload.Get()); }

	/**
	* Return a const pointer to the Payload stored in this data.
	*/
	template<typename T>
	const T* GetPayload() const { return static_cast<T*>(Payload.Get()); }

	/**
	* Sets the payload after construction.
	*/
	void SetPayload(FImagePixelPayloadPtr NewPayload) { Payload = NewPayload; }

	bool GetSRGB() const { return bSRGB; }
	void SetSRGB(bool InSRGB) { bSRGB = InSRGB; }

protected:

	FImagePixelData(const FIntPoint& InSize, EImagePixelType InPixelType, ERGBFormat InPixelLayout, uint8 InBitDepth, uint8 InNumChannels, FImagePixelPayloadPtr InPayload)
		: Size(InSize)
		, Type(InPixelType)
		, PixelLayout(InPixelLayout)
		, BitDepth(InBitDepth)
		, NumChannels(InNumChannels)
		, Payload(InPayload)
	{
		// FColor is sRGB by default, floats are Linear
		bSRGB = ( InPixelType == EImagePixelType::Color );
	}

private:

	/**
	 * Retrieve the raw pixel data
	 */
	virtual void RetrieveData(const void*& OutDataPtr, int64& OutSizeBytes) const = 0;
	virtual TUniquePtr<FImagePixelData> Copy() const = 0;
	virtual TUniquePtr<FImagePixelData> Move() = 0;

	/** The size of the pixel data */
	FIntPoint Size;

	/** The type of the derived data */
	EImagePixelType Type;

	/** The pixel layout of this data */
	ERGBFormat PixelLayout;

	/** The number of bits per each channel of color in the data */
	uint8 BitDepth;

	/** Number of channels in the data */
	uint8 NumChannels;

	/** Is FColor SRGB or Linear?  Floats are always Linear and ignore this */
	bool bSRGB;

	/** Optional user-specified payload */
	FImagePixelPayloadPtr Payload;
};

/**
 * Templated pixel data - currently supports FColor, FFloat16Color and FLinearColor
 */
template<typename PixelType>
struct TImagePixelData : FImagePixelData
{
	TArray64<PixelType> Pixels;

	TImagePixelData(const FIntPoint& InSize)
		: FImagePixelData(InSize, TImagePixelDataTraits<PixelType>::PixelType, TImagePixelDataTraits<PixelType>::PixelLayout, TImagePixelDataTraits<PixelType>::BitDepth, TImagePixelDataTraits<PixelType>::NumChannels, nullptr)
	{}

	TImagePixelData(const FIntPoint& InSize, TArray64<PixelType>&& InPixels)
		: FImagePixelData(InSize, TImagePixelDataTraits<PixelType>::PixelType, TImagePixelDataTraits<PixelType>::PixelLayout, TImagePixelDataTraits<PixelType>::BitDepth, TImagePixelDataTraits<PixelType>::NumChannels, nullptr)
		, Pixels(MoveTemp(InPixels))
	{}

	TImagePixelData(const FIntPoint& InSize, FImagePixelPayloadPtr InPayload)
		: FImagePixelData(InSize, TImagePixelDataTraits<PixelType>::PixelType, TImagePixelDataTraits<PixelType>::PixelLayout, TImagePixelDataTraits<PixelType>::BitDepth, TImagePixelDataTraits<PixelType>::NumChannels, InPayload)
	{}

	TImagePixelData(const FIntPoint& InSize, TArray64<PixelType>&& InPixels, FImagePixelPayloadPtr InPayload)
		: FImagePixelData(InSize, TImagePixelDataTraits<PixelType>::PixelType, TImagePixelDataTraits<PixelType>::PixelLayout, TImagePixelDataTraits<PixelType>::BitDepth, TImagePixelDataTraits<PixelType>::NumChannels, InPayload)
		, Pixels(MoveTemp(InPixels))
	{}

	virtual TUniquePtr<FImagePixelData> Move() override
	{
		return MakeUnique<TImagePixelData<PixelType>>(MoveTemp(*this));
	}

	virtual TUniquePtr<FImagePixelData> Copy() const override
	{
		return MakeUnique<TImagePixelData<PixelType>>(*this);
	}

	virtual void RetrieveData(const void*& OutDataPtr, int64& OutSizeBytes) const override
	{
		OutDataPtr = static_cast<const void*>(&Pixels[0]);
		OutSizeBytes = Pixels.Num() * sizeof(PixelType);
	}
};

template<> struct TImagePixelDataTraits<FColor>
{
	static const ERGBFormat PixelLayout = ERGBFormat::BGRA;
	static const EImagePixelType PixelType = EImagePixelType::Color;

	enum { BitDepth = 8, NumChannels = 4 };
};

template<> struct TImagePixelDataTraits<FFloat16Color>
{
	static const ERGBFormat PixelLayout = ERGBFormat::RGBAF;
	static const EImagePixelType PixelType = EImagePixelType::Float16;

	enum { BitDepth = 16, NumChannels = 4 };
};

template<> struct TImagePixelDataTraits<FLinearColor>
{
	static const ERGBFormat PixelLayout = ERGBFormat::RGBAF;
	static const EImagePixelType PixelType = EImagePixelType::Float32;

	enum { BitDepth = 32, NumChannels = 4 };
};

