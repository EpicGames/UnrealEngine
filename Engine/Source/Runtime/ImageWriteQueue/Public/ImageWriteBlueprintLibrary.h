// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "ImageWriteTypes.h"
#include "ImageWriteBlueprintLibrary.generated.h"

struct FImagePixelData;
class UTexture;

typedef TFunction<void(TUniquePtr<FImagePixelData>&&)> FOnPixelsReady;

DECLARE_DYNAMIC_DELEGATE_OneParam(FOnImageWriteComplete, bool, bSuccess);

/**
 * Options specific to writing image files to disk
 */
USTRUCT(BlueprintType)
struct FImageWriteOptions
{
	GENERATED_BODY()

	FImageWriteOptions()
		: Format(EDesiredImageFormat::EXR)
		, bOverwriteFile(true)
		, bAsync(true)
	{
		CompressionQuality = 0;
	}

	/** The desired output image format to write to disk */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Image")
	EDesiredImageFormat Format;

	/** A callback to invoke when the image has been written, or there was an error */
	UPROPERTY(BlueprintReadWrite, Category="Image")
	FOnImageWriteComplete OnComplete;

	/** An image format specific compression setting. Either 0 (Default) or 1 (Uncompressed) for EXRs, or a value between 0 and 100. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Image")
	int32 CompressionQuality;

	/** Whether to overwrite the image if it already exists */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Image")
	bool bOverwriteFile;

	/** Whether to perform the writing asynchronously, or to block the game thread until it is complete */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Image")
	bool bAsync;

	/** A native completion callback that will be called in addition to the dynamic one above. */
	TFunction<void(bool)> NativeOnComplete;
};

/**
 * Function library containing utility methods for writing images on a global async queue
 */
UCLASS()
class UImageWriteBlueprintLibrary : public UBlueprintFunctionLibrary
{
public:

	GENERATED_BODY()

	static bool IMAGEWRITEQUEUE_API ResolvePixelData(UTexture* Texture, const FOnPixelsReady& OnPixelsReady);

	/**
	 * Export the specified texture to disk
	 *
	 * @param Texture         The texture or render target to export
	 * @param Filename        The filename on disk to save as
	 * @param Options         Parameters defining the various export options
	 */
	UFUNCTION(BlueprintCallable, Category=Texture, meta=(ScriptMethod))
	static IMAGEWRITEQUEUE_API void ExportToDisk(UTexture* Texture, const FString& Filename, const FImageWriteOptions& Options);
};