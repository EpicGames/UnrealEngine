// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/Future.h"
#include "HAL/Platform.h"
#include "ImageWriteTask.h"
#include "Logging/LogMacros.h"
#include "Logging/LogMacros.h"
#include "Misc/CoreDefines.h"
#include "Modules/ModuleInterface.h"
#include "Templates/Function.h"
#include "Templates/UniquePtr.h"

DECLARE_LOG_CATEGORY_EXTERN(LogImageWriteQueue, Warning, Warning);

class IImageWriteTaskBase;

/**
 * Public interface for an asynchronous queue of work dedicated to writing images to disk
 *
 * Concurrency metrics are controllable by ImageWriteQueue.MaxConcurrency and ImageWriteQueue.MaxQueueSize
 * Dispatched tasks can contain callbacks that are called on the main thread when completed.
 * It is possible to wait on completion of the current queue state by creating a 'fence' that can be waited upon
 */
class IImageWriteQueue
{
public:

	virtual ~IImageWriteQueue()= default;

	/**
	 * (thread-safe) Enqueue a new asynchronous image write task.
	 *
	 * @param InTask                 A unique pointer to a task to perform on a thread when available. Pass with MoveTemp().
	 * @param bInBlockIfAtCapacity   Wait until the number of pending tasks does not exceed the queue capacity. If false and the number of pending tasks does exceed, the function will return and will not enqueue the task.
	 * @return A future to the completion state of the task (success or failure), or an invalid future in the case where the task could not be dispatched
	 */
	virtual TFuture<bool> Enqueue(TUniquePtr<IImageWriteTaskBase>&& InTask, bool bInBlockIfAtCapacity = true) = 0;


	/**
	 * (thread-safe) Create a fence at the current position in the queue. The future and callback will be invoked when all existing tasks in the queue have been completed.
	 * @note: Where the queue is empty, the future will be immediately fulfilled, and callback invoked on the next main thread tick.
	 *
	 * @param InOnFenceReached       A callback to be invoked when the fence has been reached (ie _all_ work ahead of it in the queue has been completed)
	 * @return A future that is fulfilled when the current state of the queue has been completely finished
	 */
	virtual TFuture<void> CreateFence(const TFunction<void()>& InOnFenceReached = TFunction<void()>()) = 0;


	/**
	 * (thread-safe) Query the number of tasks currently pending or in progress
	 */
	virtual int32 GetNumPendingTasks() const = 0;
};

/**
 * Module implementation that returns a write queue. Access is only via the module interface to ensure that
 * the queue is flushed correctly on shutdown
 */
class IImageWriteQueueModule : public IModuleInterface
{
public:

	/**
	 * Access a global queue of image writing tasks
	 */
	virtual IImageWriteQueue& GetWriteQueue() = 0;
};
