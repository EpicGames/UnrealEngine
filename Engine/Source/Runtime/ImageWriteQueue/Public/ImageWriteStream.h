// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "HAL/CriticalSection.h"
#include "ImagePixelData.h"
#include "Misc/CoreDefines.h"
#include "Templates/Function.h"
#include "Templates/UniquePtr.h"
#include "Templates/UnrealTemplate.h"

struct FImagePixelData;
struct FImageStreamEndpoint;

/**
 * A pipe that receives image data and forwards it onto 0 or more end points, copying the buffer as few times as possible
 */
struct FImagePixelPipe
{
	/**
	 * Default constructor (an empty pipe)
	 */
	FImagePixelPipe()
	{}

	/**
	 * Define a new pipe with a single initial endpoint
	 */
	FImagePixelPipe(const TFunction<void(TUniquePtr<FImagePixelData>&&)>& InEndpoint)
	{
		AddEndpoint(InEndpoint);
	}

	/**
	 * Push the specified pixel data onto this pipe
	 *
	 * @param InImagePixelData         The data to push through this pipe
	 */
	IMAGEWRITEQUEUE_API void Push(TUniquePtr<FImagePixelData>&& InImagePixelData);

	/**
	 * Add a new end point handler to this pipe.
	 *
	 * @param InEndpoint               The new endpoint to add. Potentially used on any thread.
	 */
	IMAGEWRITEQUEUE_API void AddEndpoint(TUniquePtr<FImageStreamEndpoint>&& InEndpoint);

	/**
	 * Add a new end point handler to this pipe as a functor.
	 *
	 * @param InHandler                A handler function implemented as an anonymous functor. Potentially called on any thread.
	 */
	IMAGEWRITEQUEUE_API void AddEndpoint(const TFunction<void(TUniquePtr<FImagePixelData>&& )>& InHandler);

	/**
	 * Access this pipe's current set of end points.
	 * Warning: Not thread-safe - should only be called where no other modification to the end points can be happening.
	 */
	TArrayView<const TUniquePtr<FImageStreamEndpoint>> GetEndPoints() const
	{
		return EndPoints;
	}

	/** Boolean flag used to request 32-bit image pixel data, false by default. */
	std::atomic_bool bIsExpecting32BitPixelData = false;

private:

	/** A lock to protect the end points array */
	FCriticalSection EndPointLock;

	/** array of endpoints to be called in order */
	TArray<TUniquePtr<FImageStreamEndpoint>> EndPoints;
};



/**
 * Stream end-point that receives a copy of image data from a thread
 */
struct FImageStreamEndpoint
{
	virtual ~FImageStreamEndpoint()= default;

	/**
	 * Pipe the specified image data onto this end point
	 *
	 * @param InOwnedImage       Image data to pass through this end point.
	 */
	IMAGEWRITEQUEUE_API void PipeImage(TUniquePtr<FImagePixelData>&& InOwnedImage);

private:

	/**
	 * Implemented in derived classes to handle image data being received
	 */
	virtual void OnImageReceived(TUniquePtr<FImagePixelData>&& InOwnedImage) {}
};
