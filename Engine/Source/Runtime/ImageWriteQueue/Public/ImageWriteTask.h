// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ContainersFwd.h"
#include "Containers/UnrealString.h"
#include "HAL/Platform.h"
#include "IImageWrapper.h"
#include "ImagePixelData.h"
#include "Math/Color.h"
#include "Math/Float16.h"
#include "Math/Float16Color.h"
#include "Math/IntPoint.h"
#include "Misc/AssertionMacros.h"
#include "Misc/CoreDefines.h"
#include "Templates/Function.h"
#include "Templates/UniquePtr.h"

template<typename> struct TImageDataTraits;

typedef TUniqueFunction<void(FImagePixelData*)> FPixelPreProcessor;

/**
 * Interface that is implemented in order to be able to asynchronously write images to disk
 */
class IImageWriteTaskBase
{
public:

	/**
	 * Virtual destruction
	 */
	virtual ~IImageWriteTaskBase() = default;

	/**
	 * 
	 */
	virtual bool RunTask() = 0;

	/**
	 * 
	 */
	virtual void OnAbandoned() = 0;
};

class FImageWriteTask
	: public IImageWriteTaskBase
{
public:

	/** The filename to write to */
	FString Filename;

	/** The desired image format to write out */
	EImageFormat Format;

	/** A compression quality setting specific to the desired image format */
	int32 CompressionQuality;

	/** True if this task is allowed to overwrite an existing file, false otherwise. */
	bool bOverwriteFile;

	/** A function to invoke on the game thread when the task has completed */
	TFunction<void(bool)> OnCompleted;

	/** The actual write operation. */
	TUniquePtr<FImagePixelData> PixelData;

	/** Array of preprocessors to apply serially to the pixel data when this task is executed. */
	TArray<FPixelPreProcessor> PixelPreProcessors;

	FImageWriteTask()
		: Format(EImageFormat::BMP)
		, CompressionQuality((int32)EImageCompressionQuality::Default)
		, bOverwriteFile(true)
	{}

public:

	IMAGEWRITEQUEUE_API virtual bool RunTask() override final;
	IMAGEWRITEQUEUE_API virtual void OnAbandoned() override final;

	IMAGEWRITEQUEUE_API void AddPreProcessorToSetAlphaOpaque();

private:

	/**
	 * Run the task, attempting to write out the raw data using the currently specified parameters
	 *
	 * @return true on success, false on any failure
	 */
	bool WriteToDisk();

	/**
	 * Ensures that the desired output filename is writable, deleting an existing file if bOverwriteFile is true
	 *
	 * @return True if the file is writable and the task can proceed, false otherwise
	 */
	bool EnsureWritableFile();

	/**
	 * Initialize the specified image wrapper with our raw data, ready for writing
	 *
	 * @param InWrapper      The wrapper to initialize with our data
	 * @param WrapperFormat  The desired image format to write out
	 * @return true on success, false on any failure
	 */
	bool InitializeWrapper(IImageWrapper* InWrapper, EImageFormat WrapperFormat);


	/**
	 * Run over all the processors for the pixel data
	 */
	void PreProcess();
};


/**
 * A pixel preprocessor for use with FImageWriteTask::PixelPreProcessor that overwrites the alpha channel with a fixed value as part of the threaded work
 *
 * DEPRECATED.  Prefer AddPreProcessorToSetAlphaOpaque.
 */
template<typename PixelType> struct TAsyncAlphaWrite;

template<>
struct TAsyncAlphaWrite<FColor> // prefer AddPreProcessorToSetAlphaOpaque
{
	uint8 Alpha;
	TAsyncAlphaWrite(uint8 InAlpha) : Alpha(InAlpha) {}

	void operator()(FImagePixelData* PixelData)
	{
		check(PixelData->GetType() == EImagePixelType::Color);

		TImagePixelData<FColor>* ColorData = static_cast<TImagePixelData<FColor>*>(PixelData);
		for (FColor& Pixel : static_cast<TImagePixelData<FColor>*>(PixelData)->Pixels)
		{
			Pixel.A = Alpha;
		}
	}
};

template<>
struct TAsyncAlphaWrite<FFloat16Color> // prefer AddPreProcessorToSetAlphaOpaque
{
	FFloat16 Alpha;
	TAsyncAlphaWrite(float InAlpha) : Alpha(InAlpha) {}

	void operator()(FImagePixelData* PixelData)
	{
		check(PixelData->GetType() == EImagePixelType::Float16);

		TImagePixelData<FFloat16Color>* Float16ColorData = static_cast<TImagePixelData<FFloat16Color>*>(PixelData);
		for (FFloat16Color& Pixel : Float16ColorData->Pixels)
		{
			Pixel.A = Alpha;
		}
	}
};

template<>
struct TAsyncAlphaWrite<FLinearColor> // prefer AddPreProcessorToSetAlphaOpaque
{
	float Alpha;
	TAsyncAlphaWrite(float InAlpha) : Alpha(InAlpha) {}

	void operator()(FImagePixelData* PixelData)
	{
		check(PixelData->GetType() == EImagePixelType::Float32);

		TImagePixelData<FLinearColor>* LinearColorData = static_cast<TImagePixelData<FLinearColor>*>(PixelData);
		for (FLinearColor& Pixel : LinearColorData->Pixels)
		{
			Pixel.A = Alpha;
		}
	}
};

/**
 * A pixel preprocessor for use with FImageWriteTask::PixelPreProcessor that inverts the alpha channel as part of the threaded work
 *
 * DEPRECATED.  This is not used that I can see; if it is, make something like AddPreProcessorToSetAlphaOpaque where the implementation
 *   is in a C file and handles all formats, not a template in a header.
 */
template<typename PixelType> struct TAsyncAlphaInvert;

template<>
struct TAsyncAlphaInvert<FColor>
{
	void operator()(FImagePixelData* PixelData)
	{
		check(PixelData->GetType() == EImagePixelType::Color);

		TImagePixelData<FColor>* ColorData = static_cast<TImagePixelData<FColor>*>(PixelData);
		for (FColor& Pixel : static_cast<TImagePixelData<FColor>*>(PixelData)->Pixels)
		{
			Pixel.A = 255 - Pixel.A;
		}
	}
};

template<>
struct TAsyncAlphaInvert<FFloat16Color>
{
	void operator()(FImagePixelData* PixelData)
	{
		check(PixelData->GetType() == EImagePixelType::Float16);

		TImagePixelData<FFloat16Color>* Float16ColorData = static_cast<TImagePixelData<FFloat16Color>*>(PixelData);
		for (FFloat16Color& Pixel : Float16ColorData->Pixels)
		{
			Pixel.A = FFloat16(1.f - Pixel.A.GetFloat());
		}
	}
};

template<>
struct TAsyncAlphaInvert<FLinearColor>
{
	void operator()(FImagePixelData* PixelData)
	{
		check(PixelData->GetType() == EImagePixelType::Float32);

		TImagePixelData<FLinearColor>* LinearColorData = static_cast<TImagePixelData<FLinearColor>*>(PixelData);
		for (FLinearColor& Pixel : LinearColorData->Pixels)
		{
			Pixel.A = 1.f - Pixel.A;
		}
	}
};
