// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ImageWriteTypes.generated.h"

enum class EImageFormat : int8;

UENUM()
enum class EDesiredImageFormat : uint8
{
	PNG,
	JPG,
	BMP,
	EXR,
};

IMAGEWRITEQUEUE_API EImageFormat ImageFormatFromDesired(EDesiredImageFormat In);